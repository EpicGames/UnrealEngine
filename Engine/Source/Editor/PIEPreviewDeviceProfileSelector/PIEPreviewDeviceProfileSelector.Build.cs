// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    [SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class PIEPreviewDeviceProfileSelector : ModuleRules
	{
        public PIEPreviewDeviceProfileSelector(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePathModuleNames.Add("Engine");

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				    "CoreUObject",
                    "Json",
                }
                );

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
                    "PIEPreviewDeviceSpecification",
                    "RHI",
					"JsonUtilities",
					"MaterialShaderQualitySettings",
					"Slate",
					"SlateCore",
                    "ApplicationCore",
					"Engine",
					"EditorFramework",
					"UnrealEd",
                }
			);
		}
	}
}
