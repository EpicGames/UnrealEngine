// Copyright Epic Games, Inc. All Rights Reserved.

#include "PIEPreviewDevice.h"

#include "HAL/IConsoleManager.h"
#include "CoreGlobals.h"
#include "RHI.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Misc/ConfigCacheIni.h"
#include "IDeviceProfileSelectorModule.h"
#include "Modules/ModuleManager.h"
#include "MaterialShaderQualitySettings.h"
#include "Android/AndroidWindowUtils.h"
#include "GenericPlatform/GenericApplication.h"
#include "Engine/Texture2D.h"
#include "ImageUtils.h"
#include "HAL/PlatformApplicationMisc.h"
#include "SEditorViewport.h"
#include "Widgets/SWindow.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

static const int32 g_JSON_VALUE_NOT_SET = 0;

FPIEPreviewDevice::FPIEPreviewDevice()
{
	DeviceSpecs = MakeShareable(new FPIEPreviewDeviceSpecifications());
}

void FPIEPreviewDevice::ShutdownDevice()
{
	if (BezelTexture != nullptr && BezelTexture->IsValidLowLevel())
	{
		BezelTexture->RemoveFromRoot();
	}
}

void FPIEPreviewDevice::ComputeViewportSize(const bool bClampWindowSize)
{
	int32 ScreenWidth, ScreenHeight;
	ComputeDeviceResolution(ScreenWidth, ScreenHeight);

	FPIEPreviewDeviceBezelViewportRect ViewportRect = DeviceSpecs->BezelProperties.BezelViewportRect;

	if (IsDeviceFlipped())
	{
		Swap(ScreenWidth, ScreenHeight);
		Swap(ViewportRect.Width, ViewportRect.Height);
		Swap(ViewportRect.X, ViewportRect.Y);
		if (BezelTexture != nullptr)
		{
			// image is rotated counter-clockwise so the top coordinate is measured from what used to be the right edge.
			ViewportRect.Y = BezelTexture->GetSizeX() - ViewportRect.Y - ViewportRect.Height;
		}
	}

	float ScaleX = (float)ScreenWidth / (float)ViewportRect.Width;
	float ScaleY = (float)ScreenHeight / (float)ViewportRect.Height;

	float BezelScaleFactor = 1.0f / DPIScaleFactor;

 	// compute widow size
	WindowWidth = ScreenWidth;
	WindowHeight = ScreenHeight + FMath::TruncToInt32((float)WindowTitleBarSize * DPIScaleFactor);

	// compute viewport margin
	if (bShowBezel && BezelTexture != nullptr)
	{
		// compute widow size
		WindowWidth += FMath::TruncToInt32(2.0f * ViewportRect.X * ScaleX);
		WindowHeight += FMath::TruncToInt32(2.0f * ViewportRect.Y * ScaleY);

		ViewportRect.X = FMath::RoundToInt(ViewportRect.X * BezelScaleFactor);
		ViewportRect.Y = FMath::RoundToInt(ViewportRect.Y * BezelScaleFactor);
		ViewportRect.Width = FMath::RoundToInt(ViewportRect.Width * BezelScaleFactor);
		ViewportRect.Height = FMath::RoundToInt(ViewportRect.Height * BezelScaleFactor);

		ViewportMargin.Left = (float)ViewportRect.X;
		ViewportMargin.Top = (float)ViewportRect.Y;

		int32 BezelWidth = IsDeviceFlipped() ? BezelTexture->GetSizeY() : BezelTexture->GetSizeX();
		int32 BezelHeight = IsDeviceFlipped() ? BezelTexture->GetSizeX() : BezelTexture->GetSizeY();

		BezelWidth = FMath::RoundToInt(BezelWidth * BezelScaleFactor);
		BezelHeight = FMath::RoundToInt(BezelHeight * BezelScaleFactor);

		ViewportMargin.Right = (float)(BezelWidth - ViewportRect.Width - ViewportRect.X);
		ViewportMargin.Bottom = (float)(BezelHeight - ViewportRect.Height - ViewportRect.Y);

		ViewportMargin = ViewportMargin * FMargin(ScaleX, ScaleY);
	}
	else
	{
		ViewportMargin = FMargin(0.0f);
	}

	// if necessary constrain the window inside the desktop boundaries
	if (bClampWindowSize)
	{
		FDisplayMetrics DisplayMetrics;
		FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);

		auto DesktopWidth = DisplayMetrics.VirtualDisplayRect.Right - DisplayMetrics.VirtualDisplayRect.Left;
		auto DesktopHeight = DisplayMetrics.VirtualDisplayRect.Bottom - DisplayMetrics.VirtualDisplayRect.Top;

		if (WindowWidth > DesktopWidth)
		{
			float ScaleFactor = (float)DesktopWidth / (float)WindowWidth;
			ScaleX *= ScaleFactor;
			ScaleY *= ScaleFactor;

			WindowWidth = DesktopWidth;
			WindowHeight = FMath::TruncToInt32((float)WindowHeight * ScaleFactor);

			ViewportMargin = ViewportMargin * ScaleFactor;
		}
		if (WindowHeight > DesktopHeight)
		{
			float ScaleFactor = (float)DesktopHeight / (float)WindowHeight;
			ScaleX *= ScaleFactor;
			ScaleY *= ScaleFactor;

			WindowWidth = FMath::TruncToInt32((float)WindowWidth * ScaleFactor);
			WindowHeight = DesktopHeight;

			ViewportMargin = ViewportMargin * ScaleFactor;
		}
	}
}

void FPIEPreviewDevice::GetDeviceDefaultResolution(int32& Width, int32& Height)
{
	Width = DeviceSpecs->ResolutionX;

	if (DeviceSpecs->ResolutionYImmersiveMode != g_JSON_VALUE_NOT_SET)
	{
		DeviceSpecs->ResolutionY = DeviceSpecs->ResolutionYImmersiveMode;
	}

	Height = DeviceSpecs->ResolutionY;
}

void FPIEPreviewDevice::ComputeContentScaledResolution(int32& Width, int32& Height)
{
	GetDeviceDefaultResolution(Width, Height);

	if (!GetIgnoreMobileContentScaleFactor())
	{
		switch (DeviceSpecs->DevicePlatform)
		{
			case EPIEPreviewDeviceType::Android:
			{
				ERHIFeatureLevel::Type DeviceFeatureLevel = GetPreviewDeviceFeatureLevel();
				AndroidWindowUtils::ApplyContentScaleFactor(Width, Height);
			}
			break;

			case EPIEPreviewDeviceType::IOS:
			{
				static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.MobileContentScaleFactor"));
				float RequestedContentScaleFactor = CVar->GetFloat();

				if (FMath::IsNearlyZero(RequestedContentScaleFactor))
				{
					RequestedContentScaleFactor = DeviceSpecs->IOSProperties.NativeScaleFactor;
				}

				Width = FMath::TruncToInt32((float)Width * RequestedContentScaleFactor);
				Height = FMath::TruncToInt32((float)Height * RequestedContentScaleFactor);
			}
			break;

			default:
			break;
		} //end switch
	}// end if (!bIgnoreContentScaleFactor)
}

void FPIEPreviewDevice::ComputeDeviceResolution(int32& Width, int32& Height)
{
	ComputeContentScaledResolution(Width, Height);

	Width = FMath::TruncToInt32((float)Width * ResolutionScaleFactor);
	Height = FMath::TruncToInt32((float)Height * ResolutionScaleFactor);
}

void FPIEPreviewDevice::DetermineScreenOrientationRequirements(bool& bNeedPortrait, bool& bNeedLandscape)
{
	switch (DeviceSpecs->DevicePlatform)
	{
		case EPIEPreviewDeviceType::Android:
		{
			FString Orientation;
			GConfig->GetString(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("Orientation"), Orientation, GEngineIni);
			if (Orientation.ToLower().Equals("portrait") || Orientation.ToLower().Equals("reverseportrait") || Orientation.ToLower().Equals("sensorportrait"))
			{
				bNeedPortrait = true;
			}
			else if (Orientation.ToLower().Equals("landscape") || Orientation.ToLower().Equals("reverselandscape") || Orientation.ToLower().Equals("sensorlandscape"))
			{
				bNeedLandscape = true;
			}
			else if (Orientation.ToLower().Equals("sensor") || Orientation.ToLower().Equals("fullsensor"))
			{
				bNeedPortrait = true;
				bNeedLandscape = true;
			}
			else
			{
				bNeedPortrait = true;
				bNeedLandscape = true;
			}
		}
		break;

		case EPIEPreviewDeviceType::IOS:
		{
			bool bSupportsPortraitOrientation;
			bool bSupportsUpsideDownOrientation;
			bool bSupportsLandscapeLeftOrientation;
			bool bSupportsLandscapeRightOrientation;
			GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsPortraitOrientation"), bSupportsPortraitOrientation, GEngineIni);
			GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsUpsideDownOrientation"), bSupportsUpsideDownOrientation, GEngineIni);
			GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsLandscapeLeftOrientation"), bSupportsLandscapeLeftOrientation, GEngineIni);
			GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsLandscapeRightOrientation"), bSupportsLandscapeRightOrientation, GEngineIni);
			bNeedPortrait = bSupportsPortraitOrientation || bSupportsUpsideDownOrientation;
			bNeedLandscape = bSupportsLandscapeLeftOrientation || bSupportsLandscapeRightOrientation;
		}
		break;
	}
}

ERHIFeatureLevel::Type FPIEPreviewDevice::GetPreviewDeviceFeatureLevel() const
{
	switch (DeviceSpecs->DevicePlatform)
	{
		case EPIEPreviewDeviceType::Android:
		{
			FString SubVersion;
			// Check for ES3.1+ support from GLVersion, TODO: check other ES31 feature level constraints, see android's PlatformInitOpenGL
			const bool bDeviceSupportsES31 = DeviceSpecs->AndroidProperties.GLVersion.Split(TEXT("OpenGL ES 3."), nullptr, &SubVersion) && FCString::Atoi(*SubVersion) >= 1;

			// check the project's gles support:
			bool bProjectBuiltForES31 = false;
			GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bBuildForES31"), bProjectBuiltForES31, GEngineIni);

			// Android Preview Device is currently expected to work on gles.
			check(bDeviceSupportsES31 && bProjectBuiltForES31);

			return ERHIFeatureLevel::ES3_1;
		}
		case EPIEPreviewDeviceType::IOS:
		case EPIEPreviewDeviceType::TVOS:
		{
			bool bProjectBuiltForMetal = false;
			GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetal"), bProjectBuiltForMetal, GEngineIni);
			// at least one of these should be valid!
			check(bProjectBuiltForMetal);

			return ERHIFeatureLevel::ES3_1;
		}
		case EPIEPreviewDeviceType::Switch:
		{
			// taken from FNVNDynamicRHI::FNVNDynamicRHI()
			// use a local ini file, so that Windows can read settings that are in SwitchEngine.ini files...
			FConfigFile SwitchSettings;
			FConfigCacheIni::LoadLocalIniFile(SwitchSettings, TEXT("Engine"), true, TEXT("Switch"));
			bool bSupportDesktopRenderer = false;
			SwitchSettings.GetBool(TEXT("/Script/SwitchRuntimeSettings.SwitchRuntimeSettings"), TEXT("bSupportDesktopRenderer"), bSupportDesktopRenderer);

			if (bSupportDesktopRenderer)
			{
				return ERHIFeatureLevel::SM5;
			}
			else
			{
				return ERHIFeatureLevel::ES3_1;
			}
		}
	}

	checkNoEntry();
	return  ERHIFeatureLevel::Num;
}

void FPIEPreviewDevice::SetupDevice(const int32 InWindowTitleBarSize)
{
	WindowTitleBarSize = InWindowTitleBarSize;

	// set initial scale factor
	ResolutionScaleFactor = 1.0f;

	// compute bezel file path
	FString BezelPath = FPaths::EngineContentDir() + TEXT("Editor/PIEPreviewDeviceSpecs/");
	if (DeviceSpecs->DevicePlatform == EPIEPreviewDeviceType::Android)
	{
		BezelPath += TEXT("Android/");
	}
	else if (DeviceSpecs->DevicePlatform == EPIEPreviewDeviceType::IOS)
	{
		BezelPath += TEXT("iOS/");
	}
	BezelPath += DeviceSpecs->BezelProperties.DeviceBezelFile;

	// load the bezel texture
	BezelTexture = FImageUtils::ImportFileAsTexture2D(BezelPath);
	if (BezelTexture != nullptr)
	{
		BezelTexture->AddToRoot();
	}
	
	// if we have invalid/uninitialized viewport values use the values provided as native device resolution
	FPIEPreviewDeviceBezelViewportRect& ViewportRect = DeviceSpecs->BezelProperties.BezelViewportRect;
	if (BezelTexture == nullptr || ViewportRect.Width == 0 || ViewportRect.Height == 0)
	{
		ViewportRect.X = 0;
		ViewportRect.Y = 0;
		ViewportRect.Width = GetDeviceSpecs()->ResolutionX;
		ViewportRect.Height = GetDeviceSpecs()->ResolutionY;
	}

	// check rotation functionalities
	bool bPortrait = false, bLandscape = false;
	DetermineScreenOrientationRequirements(bPortrait, bLandscape);

	bAllowRotation = (bPortrait && bLandscape);

	// determine current orientation
	bool bSwitchOrientation = bLandscape && DeviceSpecs->ResolutionY > DeviceSpecs->ResolutionX;
	bSwitchOrientation |= !bLandscape && bPortrait && DeviceSpecs->ResolutionX > DeviceSpecs->ResolutionY;

	if (bSwitchOrientation)
	{
		SwitchOrientation(true);
	}
}

void FPIEPreviewDevice::ApplyRHIPrerequisitesOverrides() const
{
	RHISetMobilePreviewFeatureLevel(GetPreviewDeviceFeatureLevel());
}

void FPIEPreviewDevice::ApplyRHIOverrides() const
{
	EShaderPlatform PreviewPlatform = SP_NumPlatforms;
	ERHIFeatureLevel::Type PreviewFeatureLevel = GetPreviewDeviceFeatureLevel();

	switch (DeviceSpecs->DevicePlatform)
	{
		case EPIEPreviewDeviceType::Android:
		{
			PreviewPlatform = SP_OPENGL_ES3_1_ANDROID;
		}
		break;

		case EPIEPreviewDeviceType::IOS:
		{
			PreviewPlatform = SP_METAL_ES3_1;
		}
		break;

		case EPIEPreviewDeviceType::Switch:
		{
			// apply r.ForwardShading based on Switch bUseForwardShading setting
			IConsoleVariable* CVarForwardShading = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ForwardShading"));
			if (CVarForwardShading)
			{
				// use a local ini file, so that Windows can read settings that are in SwitchEngine.ini files...
				FConfigFile SwitchSettings;
				FConfigCacheIni::LoadLocalIniFile(SwitchSettings, TEXT("Engine"), true, TEXT("Switch"));

				FString ShadingModeString; // "UseProjectSetting", "Deferred", or "Forward"
				if (GConfig->GetString(FPlatformProperties::GetRuntimeSettingsClassName(), TEXT("ShadingMode"), ShadingModeString, GEngineIni))
				{
					// Force forward shading
					if (ShadingModeString.Equals(TEXT("Forward"), ESearchCase::IgnoreCase))
					{
						CVarForwardShading->Set(1);
					}

					// Force deferred shading
					else if (ShadingModeString.Equals(TEXT("Deferred"), ESearchCase::IgnoreCase))
					{
						CVarForwardShading->Set(0);
					}
				}

				// No need to support deprecated configurations here because they will have been automatically updated upon opening the editor
			}
		}
		break;

		default:
		break;
	}

	if (PreviewPlatform != SP_NumPlatforms)
	{
		UMaterialShaderQualitySettings* MaterialShaderQualitySettings = UMaterialShaderQualitySettings::Get();
		FName QualityPreviewShaderPlatform = LegacyShaderPlatformToShaderFormat(PreviewPlatform);
		MaterialShaderQualitySettings->GetShaderPlatformQualitySettings(QualityPreviewShaderPlatform);
		ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel(PreviewPlatform);

		UMaterialShaderQualitySettings::Get()->SetPreviewPlatform(QualityPreviewShaderPlatform);
	}
}

bool FPIEPreviewDevice::GetSelectorPropertyValue(const FName& PropertyType, FString& PropertyValueOUT) const
{
	switch (DeviceSpecs->DevicePlatform)
	{
		case EPIEPreviewDeviceType::Android:
		{
			IDeviceProfileSelectorModule* AndroidDeviceProfileSelector = FModuleManager::LoadModulePtr<IDeviceProfileSelectorModule>("AndroidDeviceProfileSelector");
			if (AndroidDeviceProfileSelector)
			{
				return AndroidDeviceProfileSelector->GetSelectorPropertyValue(PropertyType, PropertyValueOUT);
			}
		}
		break;
	}
	return false;
}


FString FPIEPreviewDevice::GetProfile() const
{
	FString Profile;

	switch (DeviceSpecs->DevicePlatform)
	{
		case EPIEPreviewDeviceType::Android:
		{
			IDeviceProfileSelectorModule* AndroidDeviceProfileSelector = FModuleManager::LoadModulePtr<IDeviceProfileSelectorModule>("AndroidDeviceProfileSelector");
			if (AndroidDeviceProfileSelector)
			{
				FPIEAndroidDeviceProperties& AndroidProperties = DeviceSpecs->AndroidProperties;

				TMap<FString, FString> DeviceParameters;
				DeviceParameters.Add(TEXT("SRC_GPUFamily"), AndroidProperties.GPUFamily);
				DeviceParameters.Add(TEXT("SRC_GLVersion"), AndroidProperties.GLVersion);
				DeviceParameters.Add(TEXT("SRC_VulkanAvailable"), AndroidProperties.VulkanAvailable ? "true" : "false");
				DeviceParameters.Add(TEXT("SRC_VulkanVersion"), AndroidProperties.VulkanVersion);
				DeviceParameters.Add(TEXT("SRC_AndroidVersion"), AndroidProperties.AndroidVersion);
				DeviceParameters.Add(TEXT("SRC_DeviceMake"), AndroidProperties.DeviceMake);
				DeviceParameters.Add(TEXT("SRC_DeviceModel"), AndroidProperties.DeviceModel);
				DeviceParameters.Add(TEXT("SRC_DeviceBuildNumber"), AndroidProperties.DeviceBuildNumber);
				DeviceParameters.Add(TEXT("SRC_UsingHoudini"), AndroidProperties.UsingHoudini ? "true" : "false");
				DeviceParameters.Add(TEXT("SRC_Hardware"), AndroidProperties.Hardware);
				DeviceParameters.Add(TEXT("SRC_Chipset"), AndroidProperties.Chipset);
				DeviceParameters.Add(TEXT("SRC_HMDSystemName"), AndroidProperties.HMDSystemName);
				DeviceParameters.Add(TEXT("SRC_TotalPhysicalGB"), AndroidProperties.TotalPhysicalGB);
				DeviceParameters.Add(TEXT("SRC_SM5Available"), AndroidProperties.SM5Available ? "true" : "false");
				DeviceParameters.Add(TEXT("SRC_VKQuality"), TEXT(""));

				AndroidDeviceProfileSelector->SetSelectorProperties(DeviceParameters);
				FString PIEProfileName = AndroidDeviceProfileSelector->GetDeviceProfileName();
				if (!PIEProfileName.IsEmpty())
				{
					Profile = PIEProfileName;
				}
			}
			break;
		}
		case EPIEPreviewDeviceType::IOS:
		{
			FPIEIOSDeviceProperties& IOSProperties = DeviceSpecs->IOSProperties;
			Profile = IOSProperties.DeviceModel;
			break;
		}
		case EPIEPreviewDeviceType::Switch:
		{
			// load switch renderer configuration
			FConfigFile SwitchSettings;
			FConfigCacheIni::LoadLocalIniFile(SwitchSettings, TEXT("Engine"), true, TEXT("Switch"));
			bool bSupportDesktopRenderer = false;
			SwitchSettings.GetBool(TEXT("/Script/SwitchRuntimeSettings.SwitchRuntimeSettings"), TEXT("bSupportDesktopRenderer"), bSupportDesktopRenderer);
			bool bForwardShading = false;
			SwitchSettings.GetBool(TEXT("/Script/SwitchRuntimeSettings.SwitchRuntimeSettings"), TEXT("bUseForwardShading"), bForwardShading);
	
			// taken from FSwitchApplication::UpdateActiveDeviceProfile()
			Profile = DeviceSpecs->SwitchProperties.Docked ?
				((!bSupportDesktopRenderer || bForwardShading) ? TEXT("Switch_Console_Forward") : TEXT("Switch_Console_Deferred")) :
				((!bSupportDesktopRenderer || bForwardShading) ? TEXT("Switch_Handheld_Forward") : TEXT("Switch_Handheld_Deferred"));
		
			break;
		}
	}

	return Profile;
}

int32 FPIEPreviewDevice::GetWindowClientHeight() const
{
	return WindowHeight - FMath::TruncToInt32((float)WindowTitleBarSize * DPIScaleFactor);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS