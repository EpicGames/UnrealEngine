// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "PIEPreviewDeviceSpecification.h"

namespace ERHIFeatureLevel { enum Type : int; }

/**
 * FPIEPreviewDevice class stores specific device settings (FPIEPreviewDeviceSpecifications) and, on request, applies them to the Unreal runtime system
*/
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class UE_DEPRECATED(5.8, "PIEPreviewDeviceProfileSelector is deprecated and will be removed. Please use the new Preview Json System to preview Devices") FPIEPreviewDevice
{
private:
	TSharedPtr<FPIEPreviewDeviceSpecifications> DeviceSpecs;

	// the required size of the display window
	int32 WindowWidth = 0;
	int32 WindowHeight = 0;

	// window title bar size, needed to compute the final window size
	int32 WindowTitleBarSize = 0;

	// margins used to correctly position the viewport widget inside the provided bezel
	FMargin ViewportMargin;

	// whether or not this device can rotate its screen
	bool bAllowRotation = true;

	// true if we are in a 'rotated' state from the original orientation as provided in the json file
	bool bDeviceFlipped = false;

	// global scaling factor applied to the whole window
	float ResolutionScaleFactor = 1.0f;

	// DPI scale factor used in window size computations
	float DPIScaleFactor = 1.0f;

	/** when true extra window space will be allocated and the phone bezel will be rendered */
	bool bShowBezel = true;

	/** when true r.MobileContentScaleFactor is ignored */
	bool bIgnoreContentScaleFactor = false;

	class UTexture2D* BezelTexture = nullptr;

private:
	// utility function that will attempt to determine the supported device orientations
	void DetermineScreenOrientationRequirements(bool& bNeedPortrait, bool& bNeedLandscape);

	// this should provide the needed RHI feature level based on specified json parameters and Unreal system settings
	ERHIFeatureLevel::Type GetPreviewDeviceFeatureLevel() const;

	// call this function to apply specific RHI settings as specified in the json file
public:
	void ApplyRHIOverrides() const;

	// function that computes the viewport widget offset and needed window size
	void ComputeViewportSize(const bool bClampWindowSize);

public:
	FPIEPreviewDevice();

	// call this to run device setup -> load the bezel texture, compute appropriate orientation and apply device specific RHI settings
	void SetupDevice(const int32 InWindowTitleBarSize);

	void ShutdownDevice();

	// call this before RHI creation to apply needed setup overrides
	void ApplyRHIPrerequisitesOverrides() const;

	TSharedPtr<FPIEPreviewDeviceSpecifications> GetDeviceSpecs();

	// call this to retrieve the viewport widget padding inside the bezel 
	FMargin GetViewportMargin() const;

	// functions to retrieve the needed window size
	int32 GetWindowWidth() const;
	int32 GetWindowHeight() const;

	// functions to retrieve the window client area size
	int32 GetWindowClientWidth() const;
	int32 GetWindowClientHeight() const;

	// returns true if the current device was rotated from the original orientation
	bool IsDeviceFlipped() const;

	// call these functions to flag device rotation and specific windows scaling
	// a call to ComputeViewportSize() is necessary to update the window size
	void SwitchOrientation(const bool bClampWindowSize);
	void ScaleResolution(const float ScreenFactor, const float DPIFactor, const bool bClampWindowSize);
	void SetBezelVisibility(const bool bBezelVisible, const bool bClampWindowSize);

	bool GetBezelVisibility() const;

	// returns the current resolution scale factor (float ResolutionScaleFactor)
	float GetResolutionScale() const;

	// returns the default device resolution, as specified in the json file
	void GetDeviceDefaultResolution(int32& ScreenWidth, int32& ScreenHeight);

	// if true the device supports rotations
	bool IsRotationAllowed() const;

	// call this to enable or disable mobile content scale factor effects
	void SetIgnoreMobileContentScaleFactor(bool bIgnore);

	// call to check if the mobile content scale factor is taken into account
	bool GetIgnoreMobileContentScaleFactor() const;

	// utility function that computes the viewport resolution
	void ComputeDeviceResolution(int32& ScreenWidth, int32& ScreenHeight);

	// utility function that computes the resolution after applying r.MobileContentScaleFactor
	void ComputeContentScaledResolution(int32& ScreenWidth, int32& ScreenHeight);

	FString GetProfile() const;

	bool GetSelectorPropertyValue(const FName& PropertyType, FString& PropertyValueOUT) const;

	UTexture2D *GetBezelTexture();
};

FORCEINLINE TSharedPtr<FPIEPreviewDeviceSpecifications> FPIEPreviewDevice::GetDeviceSpecs()
{
	return DeviceSpecs;
}

FORCEINLINE FMargin FPIEPreviewDevice::GetViewportMargin() const
{
	return ViewportMargin;
}

FORCEINLINE int32 FPIEPreviewDevice::GetWindowWidth() const
{
	return WindowWidth;
}

FORCEINLINE int32 FPIEPreviewDevice::GetWindowHeight() const
{
	return WindowHeight;
}

FORCEINLINE int32 FPIEPreviewDevice::GetWindowClientWidth() const
{
	return WindowWidth;
}

FORCEINLINE bool FPIEPreviewDevice::IsDeviceFlipped() const
{
	return bDeviceFlipped;
}

FORCEINLINE bool FPIEPreviewDevice::IsRotationAllowed() const
{
	return bAllowRotation;
}

FORCEINLINE void FPIEPreviewDevice::SwitchOrientation(const bool bClampWindowSize)
{
	bDeviceFlipped = !bDeviceFlipped;

	ComputeViewportSize(bClampWindowSize);
}

FORCEINLINE void FPIEPreviewDevice::ScaleResolution(const float ScreenFactor, const float DPIFactor, const bool bClampWindowSize)
{
	ResolutionScaleFactor = ScreenFactor;
	DPIScaleFactor = DPIFactor;

	ComputeViewportSize(bClampWindowSize);
}

FORCEINLINE void FPIEPreviewDevice::SetBezelVisibility(const bool bBezelVisible, const bool bClampWindowSize)
{
	bShowBezel = bBezelVisible;

	ComputeViewportSize(bClampWindowSize);
}

FORCEINLINE bool FPIEPreviewDevice::GetBezelVisibility() const
{
	return bShowBezel;
}

FORCEINLINE float FPIEPreviewDevice::GetResolutionScale() const
{
	return ResolutionScaleFactor;
}

FORCEINLINE UTexture2D *FPIEPreviewDevice::GetBezelTexture()
{
	return BezelTexture;
}

FORCEINLINE void FPIEPreviewDevice::SetIgnoreMobileContentScaleFactor(bool bIgnore)
{
	bIgnoreContentScaleFactor = bIgnore;
}

FORCEINLINE bool FPIEPreviewDevice::GetIgnoreMobileContentScaleFactor() const
{
	return bIgnoreContentScaleFactor;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS