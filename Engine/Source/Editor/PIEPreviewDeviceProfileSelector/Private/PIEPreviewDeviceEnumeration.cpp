// Copyright Epic Games, Inc. All Rights Reserved.

#include "PIEPreviewDeviceEnumeration.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Paths.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

#define LOCTEXT_NAMESPACE "PIEPreviewDevice"

void FPIEPreviewDeviceContainer::EnumerateDeviceSpecifications(const FString& RootDir)
{
	RootCategory = MakeShareable(new FPIEPreviewDeviceContainerCategory(RootDir, FText()));
	DeviceSpecifications.Reset();
	DeviceSpecificationsLocalizedName.Reset();
	EnumerateDeviceSpecifications(RootCategory);
	EnumerateDeviceLocalizedNames();
}

void FPIEPreviewDeviceContainer::EnumerateDeviceSpecifications(TSharedPtr<FPIEPreviewDeviceContainerCategory> CurrentCategory)
{
	// Enumerate device spec .json files in this dir:
	const FString& CurrentDirectory = CurrentCategory->GetSubDirectoryPath();
	TArray<FString> DeviceSpecificationFileNames;
	IFileManager::Get().FindFiles(DeviceSpecificationFileNames, *(CurrentDirectory / TEXT("*.json")), true, false);

	// Store the device index range for this category.
	CurrentCategory->DeviceStartIndex = DeviceSpecifications.Num();
	for (TArray<FString>::TConstIterator It(DeviceSpecificationFileNames); It; ++It)
	{
		DeviceSpecifications.Add(FPaths::GetBaseFilename(*It));
	}
	CurrentCategory->DeviceCount = DeviceSpecifications.Num() - CurrentCategory->DeviceStartIndex;

	TArray<FString> DeviceSpecificationDirectories;
	IFileManager::Get().FindFiles(DeviceSpecificationDirectories, *(CurrentDirectory / TEXT("*")), false, true);
	for (FString& SubDir : DeviceSpecificationDirectories)
	{
		TSharedPtr<FPIEPreviewDeviceContainerCategory> SubCategory = MakeShareable(new FPIEPreviewDeviceContainerCategory(CurrentDirectory / SubDir, FText::FromString(SubDir)));
		CurrentCategory->SubCategories.Add(SubCategory);
		EnumerateDeviceSpecifications(SubCategory);
	}
}

void FPIEPreviewDeviceContainer::EnumerateDeviceLocalizedNames()
{
	const TArray<FString>& Devices = GetDeviceSpecifications();

	for (int32 DeviceIndex = 0; DeviceIndex < Devices.Num(); DeviceIndex++)
	{
		DeviceSpecificationsLocalizedName.Add(Devices[DeviceIndex]);
	}
}

const TSharedPtr<FPIEPreviewDeviceContainerCategory> FPIEPreviewDeviceContainer::FindDeviceContainingCategory(int32 DeviceIndex) const
{
	struct FLocal
	{
		static TSharedPtr<FPIEPreviewDeviceContainerCategory> FindDeviceContainingCategory(int32 DeviceIndexIn, TSharedPtr<FPIEPreviewDeviceContainerCategory> PreviewCategory)
		{
			int32 StartIndex = PreviewCategory->GetDeviceStartIndex();
			int32 EndIndex = StartIndex + PreviewCategory->GetDeviceCount();
			if (DeviceIndexIn >= PreviewCategory->GetDeviceStartIndex() && DeviceIndexIn < EndIndex)
			{
				return PreviewCategory;
			}
			else
			{
				for (const auto& SubCategory : PreviewCategory->GetSubCategories())
				{
					TSharedPtr<FPIEPreviewDeviceContainerCategory> ContainingSubcategory = FindDeviceContainingCategory(DeviceIndexIn, SubCategory);
					if (ContainingSubcategory.IsValid())
					{
						return ContainingSubcategory;
					}
				}
			}
			return nullptr;
		}
	};
	return FLocal::FindDeviceContainingCategory(DeviceIndex, GetRootCategory());
}

FName FPIEPreviewDeviceContainerCategory::GetCategoryName()
{
	return FName(*FString(CategoryDisplayName.ToString() + "_PIEDevicePreview" ));
}

FText FPIEPreviewDeviceContainerCategory::GetCategoryToolTip()
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("Device"), CategoryDisplayName);
	return FText::Format(LOCTEXT("PIEPreviewDeviceCategoryToolTip", "Preview {Device} devices"), Args);
}

#undef LOCTEXT_NAMESPACE
PRAGMA_ENABLE_DEPRECATION_WARNINGS