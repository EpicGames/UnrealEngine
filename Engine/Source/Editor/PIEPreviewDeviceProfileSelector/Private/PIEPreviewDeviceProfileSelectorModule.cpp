// Copyright Epic Games, Inc. All Rights Reserved.

#include "PIEPreviewDeviceProfileSelectorModule.h"
#include "Misc/FileHelper.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "PIEPreviewDeviceSpecification.h"
#include "JsonObjectConverter.h"
#include "MaterialShaderQualitySettings.h"
#include "RHI.h"
#include "Framework/Docking/TabManager.h"
#include "CoreGlobals.h"
#include "Modules/ModuleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Internationalization/Culture.h"
#include "PIEPreviewWindowStyle.h"
#include "PIEPreviewDevice.h"
#include "PIEPreviewWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "UnrealEngine.h"
#include "Engine/GameViewportClient.h"
#include "Engine/UserInterfaceSettings.h"
#include "PIEPreviewSettings.h"
#include "PIEPreviewWindowCoreStyle.h"
#include "Misc/CommandLine.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

DECLARE_LOG_CATEGORY_EXTERN(LogPIEPreviewDevice, Log, All); 
DEFINE_LOG_CATEGORY(LogPIEPreviewDevice);
IMPLEMENT_MODULE(FPIEPreviewDeviceModule, PIEPreviewDeviceProfileSelector);

void FPIEPreviewDeviceModule::StartupModule()
{
	// Parse the json file specified on the command line
	FString PreviewDeviceCmdLine;
	if (FParse::Value(FCommandLine::Get(), GetPreviewDeviceCommandSwitch(), PreviewDeviceCmdLine))
	{
		SetPreviewDevice(PreviewDeviceCmdLine);
	}
}

void FPIEPreviewDeviceModule::ShutdownModule()
{
	// clear delegates set in StartupModule()
	if (EngineInitCompleteDelegate.IsValid())
	{
		FCoreDelegates::OnFEngineLoopInitComplete.Remove(EngineInitCompleteDelegate);
	}

	if (ViewportCreatedDelegate.IsValid())
	{
		UGameViewportClient::OnViewportCreated().Remove(ViewportCreatedDelegate);
	}

	TSharedPtr<SPIEPreviewWindow> WindowPtr = WindowWPtr.Pin();
	if (WindowPtr.IsValid())
	{
		WindowPtr->PrepareShutdown();
	}

	if (Device.IsValid())
	{
		Device->ShutdownDevice();
	}
}

void FPIEPreviewDeviceModule::SetPreviewDevice(const FString& DeviceName)
{
	PreviewDevice = DeviceName;
	const FString Filename = FindDeviceSpecificationFilePath(PreviewDevice);

	FString Json;
	if (FFileHelper::LoadFileToString(Json, *Filename))
	{
		TSharedRef<TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(Json);
		FJsonSerializer::Deserialize(JsonReader, JsonRootObject);
	}

	InitPreviewDevice();
}

FName FPIEPreviewDeviceModule::GetPreviewPlatformName()
{
	// Here we need to parse the json directly as we have not yet initialized the UObject system
	if (JsonRootObject.IsValid())
	{
		FString DevicePlatform;
		if (JsonRootObject->TryGetStringField(TEXT("DevicePlatform"), DevicePlatform))
		{
			return FName(*DevicePlatform);
		}
	}
	return NAME_None;
}

FString const FPIEPreviewDeviceModule::GetRuntimeDeviceProfileName()
{
	if (!bInitialized)
	{
		InitPreviewDevice();
	}

	return DeviceProfile;
}

bool FPIEPreviewDeviceModule::GetSelectorPropertyValue(const FName& PropertyType, FString& PropertyValueOUT)
{ 
	return Device->GetSelectorPropertyValue(PropertyType, PropertyValueOUT);
}

void FPIEPreviewDeviceModule::InitPreviewDevice()
{
	bInitialized = true;

	// the window size will be available after all data is loaded and we'll use this callback to display it
	EngineInitCompleteDelegate = FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FPIEPreviewDeviceModule::OnEngineInitComplete);

	// to finish setup we need complete engine initialization
	ViewportCreatedDelegate = UGameViewportClient::OnViewportCreated().AddRaw(this, &FPIEPreviewDeviceModule::OnViewportCreated);

	bool bReadSuccess = ReadDeviceSpecification();
	checkf(bReadSuccess, TEXT("Unable to read PIE Preview Device specification"));

	// Do not apply RHI overrides when using the editor
	if(!GIsEditor)
	{
		Device->ApplyRHIPrerequisitesOverrides();
	}
	DeviceProfile = Device->GetProfile();
}

void FPIEPreviewDeviceModule::OnEngineInitComplete()
{
	TSharedPtr<SPIEPreviewWindow> WindowPtr = WindowWPtr.Pin();

	// TODO: Localization
	FString AppTitle = FGlobalTabmanager::Get()->GetApplicationTitle().ToString() + "Previewing: " + PreviewDevice;
	FGlobalTabmanager::Get()->SetApplicationTitle(FText::FromString(AppTitle));

	if (WindowPtr.IsValid())
	{
		int32 TitleBarSize = SPIEPreviewWindow::GetDefaultTitleBarSize();
		Device->SetupDevice(TitleBarSize);
		
		ReadWindowConfig();
		WindowPtr->PrepareWindow(InitialWindowPosition, InitialWindowScaleValue, Device);
		WindowPtr->ShowWindow();
	}
}

bool FPIEPreviewDeviceModule::ReadWindowConfig()
{
	auto* Settings = GetDefault<UPIEPreviewSettings>();
	InitialWindowScaleValue = Settings->WindowScalingFactor;

	InitialWindowPosition.Set(Settings->WindowPosX, Settings->WindowPosY);

	return Settings->WindowPosX > 0 &&  Settings->WindowPosY > 0;
}

TSharedRef<SWindow> FPIEPreviewDeviceModule::CreatePIEPreviewDeviceWindow(FVector2D ClientSize, FText WindowTitle, EAutoCenter AutoCenterType, FVector2D ScreenPosition, TOptional<float> MaxWindowWidth, TOptional<float> MaxWindowHeight)
{
	InitialWindowPosition = ScreenPosition;

	FPIEPreviewWindowCoreStyle::InitializePIECoreStyle();

	static FWindowStyle BackgroundlessStyle = FCoreStyle::Get().GetWidgetStyle<FWindowStyle>("Window");
	BackgroundlessStyle.SetBackgroundBrush(FSlateNoResource());
	TSharedRef<SPIEPreviewWindow> Window = SNew(SPIEPreviewWindow)
		.Style(&BackgroundlessStyle)
		.ClientSize(ClientSize)
		.Title(WindowTitle)
		.AutoCenter(AutoCenterType)
		.ScreenPosition(ScreenPosition)
		.MaxWidth(MaxWindowWidth)
		.MaxHeight(MaxWindowHeight)
		.FocusWhenFirstShown(true)
		.SaneWindowPlacement(AutoCenterType == EAutoCenter::None)
		.UseOSWindowBorder(false)
		.CreateTitleBar(true)
		.ShouldPreserveAspectRatio(true)
		.LayoutBorder(FMargin(0))
		.SizingRule(ESizingRule::FixedSize)
		.HasCloseButton(true)
		.SupportsMinimize(true)
		.SupportsMaximize(false)
		.bManualManageDPI(false);

 	WindowWPtr = Window;

	if (GameLayerManagerWidget.IsValid())
	{
		Window->SetGameLayerManagerWidget(GameLayerManagerWidget);
	}

	return Window;
}

void FPIEPreviewDeviceModule::UpdateDisplayResolution()
{
	TSharedPtr<SPIEPreviewWindow> WindowPtr = WindowWPtr.Pin();

	if (!Device.IsValid() || !WindowPtr.IsValid())
	{
		return;
	}

	const int32 ClientWidth = Device->GetWindowWidth();
	const int32 ClientHeight = Device->GetWindowHeight() - FMath::TruncToInt32(WindowPtr->GetTitleBarSize().Get());

	FSystemResolution::RequestResolutionChange(ClientWidth, ClientHeight, EWindowMode::Windowed);
	IConsoleManager::Get().CallAllConsoleVariableSinks();
}

void FPIEPreviewDeviceModule::OnWindowReady(TSharedRef<SWindow> Window)
{
	TSharedPtr<SPIEPreviewWindow> WindowPtr = StaticCastSharedRef<SPIEPreviewWindow>(Window);

	if (WindowPtr.IsValid())
	{
		// the window will only be displayed after the loading is complete (OnEngineInitComplete)
		WindowPtr->HideWindow();
	}

	FSlateApplication::Get().SetGameIsFakingTouchEvents(true);
}

void FPIEPreviewDeviceModule::ApplyPreviewDeviceState()
{
	if (!Device.IsValid())
	{
		return;
	}

	Device->ApplyRHIOverrides();
}

void FPIEPreviewDeviceModule::OnViewportCreated()
{
	// disable mouse viewport locking
	if (GEngine->GameViewport != nullptr)
	{
		GEngine->GameViewport->SetMouseCaptureMode(EMouseCaptureMode::NoCapture);
		GEngine->GameViewport->SetMouseLockMode(EMouseLockMode::DoNotLock);
	}
}

const FPIEPreviewDeviceContainer& FPIEPreviewDeviceModule::GetPreviewDeviceContainer()
{
	if (!EnumeratedDevices.GetRootCategory().IsValid())
	{
		EnumeratedDevices.EnumerateDeviceSpecifications(GetDeviceSpecificationContentDir());
	}
	return EnumeratedDevices;
}

FString FPIEPreviewDeviceModule::GetDeviceSpecificationContentDir()
{
	return FPaths::EngineContentDir() / TEXT("Editor") / TEXT("PIEPreviewDeviceSpecs");
}

FString FPIEPreviewDeviceModule::FindDeviceSpecificationFilePath(const FString& SearchDevice)
{
	const FPIEPreviewDeviceContainer& PIEPreviewDeviceContainer = GetPreviewDeviceContainer();
	FString FoundPath;

	int32 FoundIndex;
	if (PIEPreviewDeviceContainer.GetDeviceSpecifications().Find(SearchDevice, FoundIndex))
	{
		TSharedPtr<FPIEPreviewDeviceContainerCategory> SubCategory = PIEPreviewDeviceContainer.FindDeviceContainingCategory(FoundIndex);
		if(SubCategory.IsValid())
		{
			FoundPath = SubCategory->GetSubDirectoryPath() / SearchDevice + ".json";
		}
	}
	return FoundPath;
}

bool FPIEPreviewDeviceModule::ReadDeviceSpecification()
{
	Device = nullptr;

	if (JsonRootObject.IsValid())
	{
		// We need to initialize FPIEPreviewDeviceSpecifications early as device profiles need to be evaluated before ProcessNewlyLoadedUObjects can be called.
		CreatePackage( TEXT("/Script/PIEPreviewDeviceProfileSelector"));

		Device = MakeShareable(new FPIEPreviewDevice());

		if (!FJsonObjectConverter::JsonAttributesToUStruct(JsonRootObject->Values, FPIEPreviewDeviceSpecifications::StaticStruct(), Device->GetDeviceSpecs().Get(), 0, 0))
		{
			Device = nullptr;
		}
	}
	bool bValidDeviceSpec = Device.IsValid();
	if (!bValidDeviceSpec)
	{
		UE_LOGF(LogPIEPreviewDevice, Warning, "Could not load device specifications for preview target device '%ls'", *PreviewDevice);
	}

	return bValidDeviceSpec;
}

void FPIEPreviewDeviceModule::SetGameLayerManagerWidget(TSharedPtr<class SGameLayerManager> GameLayerManager)
{
	GameLayerManagerWidget = GameLayerManager;

	TSharedPtr<SPIEPreviewWindow> WindowPtr = WindowWPtr.Pin();
	if (WindowPtr.IsValid())
	{
		WindowPtr->SetGameLayerManagerWidget(GameLayerManager);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS