// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "IPIEPreviewDeviceModule.h"

const FString IPIEPreviewDeviceModule::GetRuntimeDeviceProfileName()
{
	return FString();
}

void IPIEPreviewDeviceModule::StartupModule()
{
}

void IPIEPreviewDeviceModule::ShutdownModule()
{
}
