// Copyright Epic Games, Inc. All Rights Reserved.

#include "PIEPreviewSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PIEPreviewSettings)

UPIEPreviewSettings::UPIEPreviewSettings(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	WindowPosX = -1;
	WindowPosY = -1;
	WindowScalingFactor = 0;
}
