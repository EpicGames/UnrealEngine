// Copyright Epic Games, Inc. All Rights Reserved.

/**
*
* A configuration class used by the PIE preview device system to save settings across sessions.
*/

#pragma once

#include "HAL/Platform.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"

#include "PIEPreviewSettings.generated.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(MinimalAPI, hidecategories = Object, config = PIEPreviewSettings)
class UE_DEPRECATED(5.8, "PIEPreviewDeviceProfileSelector is deprecated and will be removed. Please use the new Preview Json System to preview Devices") UPIEPreviewSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(config)
	int32 WindowPosX;

	UPROPERTY(config)
	int32 WindowPosY;

	UPROPERTY(config)
	float WindowScalingFactor;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS