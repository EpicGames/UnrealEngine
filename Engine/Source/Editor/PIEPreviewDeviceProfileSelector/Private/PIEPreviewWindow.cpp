// Copyright Epic Games, Inc. All Rights Reserved.

#include "PIEPreviewWindow.h"

#if WITH_EDITOR

#include "HAL/IConsoleManager.h"
#include "ImageUtils.h"
#include "Engine/Texture2D.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "SEditorViewportToolBarMenu.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Misc/ConfigCacheIni.h"
#include "Slate/SGameLayerManager.h"
#include "UnrealEngine.h"
#include "PIEPreviewSettings.h"
#include "PIEPreviewDevice.h"
#include "PIEPreviewDeviceSpecification.h"
#include "PIEPreviewWindowTitleBar.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
//***********************************************************************************
//SPIEPreviewWindow Implementation
//***********************************************************************************
SPIEPreviewWindow::SPIEPreviewWindow()
{
}

SPIEPreviewWindow::~SPIEPreviewWindow()
{
	PrepareShutdown();
}

TSharedRef<SWidget> SPIEPreviewWindow::MakeWindowTitleBar(const TSharedRef<SWindow>& Window, const TSharedPtr<SWidget>& CenterContent, EHorizontalAlignment CenterContentAlignment)
{
	WindowTitleBar = SNew(SPIEPreviewWindowTitleBar, Window, CenterContent, EHorizontalAlignment::HAlign_Center)
		.Visibility(EVisibility::SelfHitTestInvisible);

	return WindowTitleBar.ToSharedRef();
}

EHorizontalAlignment SPIEPreviewWindow::GetTitleAlignment()
{
	return EHorizontalAlignment::HAlign_Left;
}

void SPIEPreviewWindow::ComputeBezelOrientation()
{
	if (BezelImage.IsValid())
	{
 		float Width = (float)Device->GetWindowClientWidth();
 		float Height = (float)Device->GetWindowClientHeight();

		bool bBezelRotated = Device->IsDeviceFlipped();

		float ScaleX = bBezelRotated ? Width / Height : 1.0f;
		float ScaleY = bBezelRotated ? Inverse(ScaleX) : 1.0f;

		FScale2D Scale = FScale2D(ScaleX, ScaleY);
		FQuat2D Rotation = FQuat2D(bBezelRotated ? -PI / 2 : 0);
		FMatrix2x2 ImageTransformationMatrix = Concatenate(Rotation, Scale);

		BezelImage->SetRenderTransform(FSlateRenderTransform(ImageTransformationMatrix));
	}
}

TSharedRef<SWidget> SPIEPreviewWindow::BuildSettingsMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	auto ScaleDescriptionWidget = SNew(STextBlock)
		.Text(FText::FromString(TEXT("Window Scale")))
		.Justification(ETextJustify::Center);

	MenuBuilder.AddWidget(ScaleDescriptionWidget, FText());
	MenuBuilder.AddMenuSeparator();

	// create a scaling checkboxes for each scaling factor needed by the emulated device
	TArray<float>& ArrScaleFactors = Device->GetDeviceSpecs()->ScaleFactors;
	for (int32 i = 0; i < ArrScaleFactors.Num(); ++i)
	{
		const float ScaleFactor = ArrScaleFactors[i];

		FText EntryText = FText::FromString(FString::SanitizeFloat(ArrScaleFactors[i]) + TEXT("x"));
		auto IsCheckedFunction = [this, ScaleFactor]()
		{
			float WindowScaleFactor = GetWindowScaleFactor();
			ECheckBoxState CheckState = FMath::IsNearlyEqual(ScaleFactor, WindowScaleFactor) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;

			return CheckState;
		};

		auto ExecuteActionFunction = [this, ScaleFactor]()
		{
			SetWindowScaleFactor(ScaleFactor);
		};

		CreateMenuEntry(MenuBuilder, MoveTemp(EntryText), MoveTemp(IsCheckedFunction), MoveTemp(ExecuteActionFunction));
	}

	// scale to device size checkbox
	if (Device->GetDeviceSpecs()->PPI != 0)
	{
		FText EntryText = FText::FromString(TEXT("Scale to device size"));
		auto IsCheckedFunction = [this]()
		{
			float WindowScaleFactor = GetWindowScaleFactor();
			float DeviceSizeFactor = GetScaleToDeviceSizeFactor();

			ECheckBoxState CheckState = FMath::IsNearlyEqual(WindowScaleFactor, DeviceSizeFactor) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			return CheckState;
		};

		auto ExecuteActionFunction = [this]()
		{
			float WindowScaleFactor = GetScaleToDeviceSizeFactor();
			SetWindowScaleFactor(WindowScaleFactor);
		};

		CreateMenuEntry(MenuBuilder, MoveTemp(EntryText), MoveTemp(IsCheckedFunction), MoveTemp(ExecuteActionFunction));

		MenuBuilder.AddMenuSeparator();
	}

	// add bClampWindowSizeState checkbox
	{
		FText EntryText = FText::FromString(TEXT("Restrict to desktop size"));
		auto IsCheckedFunction = [this]()
		{
			bool bChecked = IsClampingWindowSize();
			return bChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		};

		auto ExecuteActionFunction = [this]()
		{
			bool bClamp = IsClampingWindowSize();
			SetClampWindowSize(!bClamp);
		};

		CreateMenuEntry(MenuBuilder, MoveTemp(EntryText), MoveTemp(IsCheckedFunction), MoveTemp(ExecuteActionFunction));
	}
	// add checkbox to handle bezel visibility
	{
		FText EntryText = FText::FromString(TEXT("Show phone bezel"));
		auto IsCheckedFunction = [this]()
		{
			bool bBezelVisibility = GetBezelVisibility();
			return bBezelVisibility ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		};

		auto ExecuteActionFunction = [this]()
		{
			FlipBezelVisibility();
		};

		CreateMenuEntry(MenuBuilder, MoveTemp(EntryText), MoveTemp(IsCheckedFunction), MoveTemp(ExecuteActionFunction));
	}

	MenuBuilder.AddMenuSeparator();
	auto ResolutionDescriptionWidget = SNew(STextBlock)
		.Text(FText::FromString(TEXT("Resolution")))
		.Justification(ETextJustify::Center);

	MenuBuilder.AddWidget(ResolutionDescriptionWidget, FText());

	// Base resolution text
	{
		auto PrintLambda = [Device = Device]()
		{
			FString ResolutionText = TEXT("Device - ");
			if (Device.IsValid())
			{
				int32 ResX, ResY;
				Device->GetDeviceDefaultResolution(ResX, ResY);
				if (Device->IsDeviceFlipped())
				{
					Swap(ResX, ResY);
				}

				ResolutionText += FString::FromInt(ResX) + TEXT("x") + FString::FromInt(ResY);
			}

			return FText::FromString(ResolutionText);
		};

		CreateTextMenuEntry(MenuBuilder, MoveTemp(PrintLambda));
	}
	// End Base resolution text

	// Resolution with content scale
	{
		auto PrintLambda = [Device = Device]()
		{
			FString ResolutionText = TEXT("Content - ");
			if (Device.IsValid())
			{
				int32 ResX, ResY;
				bool bDeviceIgnoresContentFactor = Device->GetIgnoreMobileContentScaleFactor();
				Device->SetIgnoreMobileContentScaleFactor(false);
				Device->ComputeContentScaledResolution(ResX, ResY);
				Device->SetIgnoreMobileContentScaleFactor(bDeviceIgnoresContentFactor);
				if (Device->IsDeviceFlipped())
				{
					Swap(ResX, ResY);
				}
				ResolutionText += FString::FromInt(ResX) + TEXT("x") + FString::FromInt(ResY);
			}

			return FText::FromString(ResolutionText);
		};

		CreateTextMenuEntry(MenuBuilder, MoveTemp(PrintLambda));
	}
	// Resolution with content scale

	// Displayed resolution
	{
		auto PrintLambda = [Device = Device]()
		{
			FString ResolutionText = TEXT("Window - ");
			if (Device.IsValid())
			{
				int32 ResX, ResY;
				Device->ComputeDeviceResolution(ResX, ResY);
				if (Device->IsDeviceFlipped())
				{
					Swap(ResX, ResY);
				}
				ResolutionText += FString::FromInt(ResX) + TEXT("x") + FString::FromInt(ResY);
			}

			return FText::FromString(ResolutionText);
		};

		CreateTextMenuEntry(MenuBuilder, MoveTemp(PrintLambda));
	}
	// Resolution with content scale

	return MenuBuilder.MakeWidget();
}

void SPIEPreviewWindow::CreateTextMenuEntry(class FMenuBuilder& MenuBuilder, TFunction<FText()>&& CreateTextFunction)
{
	auto Box = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		[
			SNew(STextBlock)
			.Visibility(EVisibility::HitTestInvisible)
			.Justification(ETextJustify::Center)
			.Text_Lambda(MoveTemp(CreateTextFunction))
		];

	MenuBuilder.AddWidget(Box, FText());
}

void SPIEPreviewWindow::CreateMenuEntry(FMenuBuilder& MenuBuilder, FText&& TextEntry, TFunction<ECheckBoxState()>&& IsCheckedFunction, TFunction<void()>&& ExecuteActionFunction)
{
	auto Box =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.Padding(FMargin(5.0f, 0.0f))
		[
			SNew(STextBlock)
			.Visibility(EVisibility::HitTestInvisible)
			.Text(MoveTemp(TextEntry))
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(10.0f, 0.0f))
		.HAlign(EHorizontalAlignment::HAlign_Right)
		[
			SNew(SCheckBox)
			.IsFocusable(false)
			.IsEnabled(false)
			.IsChecked_Lambda(MoveTemp(IsCheckedFunction))
		];

	MenuBuilder.AddMenuEntry(FUIAction(FExecuteAction::CreateLambda(MoveTemp(ExecuteActionFunction))), Box);
}

void SPIEPreviewWindow::CreatePIEPreviewBezelOverlay(UTexture2D* pBezelImage)
{
	if (pBezelImage != nullptr)
	{
		BezelBrush.SetResourceObject(pBezelImage);
		BezelBrush.ImageSize = FVector2D(pBezelImage->GetSizeX(), pBezelImage->GetSizeY());

		auto GetBezelVisibility = [this]()
		{
			EVisibility Visibility = EVisibility::Collapsed;

			if (Device.IsValid())
			{
				bool bVisible = Device->GetBezelVisibility();
				Visibility = bVisible ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
			}

			return Visibility;
		};

		BezelImage = SNew(SImage)
			.Image(&BezelBrush)
			.Visibility_Lambda(GetBezelVisibility)
			.RenderTransformPivot(FVector2D(.5f, .5f));

		AddOverlaySlot()
			.Padding(0, SWindowDefs::DefaultTitleBarSize, 0, 0)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				BezelImage.ToSharedRef()
			];

		ComputeBezelOrientation();
	}
}

void SPIEPreviewWindow::ValidatePosition(FVector2D& WindowPos)
{
	WindowPos.X = FMath::CeilToInt32(WindowPos.X);
	WindowPos.Y = FMath::CeilToInt32(WindowPos.Y);

	FDisplayMetrics DisplayMetrics;
	FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);

	const int32 Offset = 5;
	if ((WindowPos.X - Offset > DisplayMetrics.VirtualDisplayRect.Right) ||
		(WindowPos.X + Device->GetWindowWidth() + Offset < DisplayMetrics.VirtualDisplayRect.Left) ||
		(WindowPos.Y - Offset > DisplayMetrics.VirtualDisplayRect.Bottom) ||
		(WindowPos.Y + Device->GetWindowHeight() + Offset < DisplayMetrics.VirtualDisplayRect.Top))
	{
		WindowPos.X = (DisplayMetrics.PrimaryDisplayWorkAreaRect.Left + DisplayMetrics.PrimaryDisplayWorkAreaRect.Right) / 2;
		WindowPos.Y = (DisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom + DisplayMetrics.PrimaryDisplayWorkAreaRect.Top) / 2;
	}
}

void SPIEPreviewWindow::PrepareWindow(FVector2D WindowPosition, const float InitialScaleFactor, TSharedPtr<FPIEPreviewDevice> PreviewDevice)
{
	// we always manually handle DPI changes
	SetManualManageDPIChanges(true);

	SetDevice(PreviewDevice);

	// place window to the required position and compute its size
	ValidatePosition(WindowPosition);
	MoveWindowTo(WindowPosition);
	SetWindowScaleFactor(InitialScaleFactor, false);

	// update display resolution
	const int32 ClientWidth = Device->GetWindowWidth();
	const int32 ClientHeight = Device->GetWindowHeight() - FMath::TruncToInt32(GetTitleBarSize().Get());
	FSystemResolution::RequestResolutionChange(ClientWidth, ClientHeight, EWindowMode::Windowed);
	IConsoleManager::Get().CallAllConsoleVariableSinks();

	// the above call will reset the position of the window and set the wrong size (due to manual DPI) and we need to set it right
	MoveWindowTo(WindowPosition);
	UpdateWindow();

	// set needed event callbacks
	SetOnWindowMoved(FOnWindowMoved::CreateSP(this, &SPIEPreviewWindow::OnWindowMoved));
	HandleDPIChange = FSlateApplication::Get().OnSystemSignalsDPIChanged().AddSP(this, &SPIEPreviewWindow::OnDisplayDPIChanged);
}

void SPIEPreviewWindow::SetWindowScaleFactor(const float ScaleFactor, const bool bStore/* = true*/)
{
	WindowScalingFactor = ScaleFactor;

	// when required we will save the scaling value so it can be restored after session restart
	if (bStore)
	{
		auto* Settings = GetMutableDefault<UPIEPreviewSettings>();
		Settings->WindowScalingFactor = ScaleFactor;
		Settings->SaveConfig();
	}

	ScaleWindow(ScaleFactor);
}

void SPIEPreviewWindow::ScaleWindow(float ScaleFactor)
{
	if (!Device.IsValid())
	{
		return;
	}

	bool bScaleToDeviceSize = IsScalingToDeviceSizeFactor(ScaleFactor);
	Device->SetIgnoreMobileContentScaleFactor(bScaleToDeviceSize);

	float DPIScaleFactor = ComputeDPIScaleFactor();

	if (bScaleToDeviceSize)
	{
		ScaleFactor = ComputeScaleToDeviceSizeFactor();
		ScaleFactor /= DPIScaleFactor;
	}

	if (FMath::IsNearlyEqual(ScaleFactor, CachedScaleToDeviceFactor) && FMath::IsNearlyEqual(DPIScaleFactor, CachedDPIScaleFactor))
	{
		return;
	}

	CachedScaleToDeviceFactor = ScaleFactor;
	CachedDPIScaleFactor = DPIScaleFactor;

	SetDPIScaleFactor(CachedDPIScaleFactor);

	if (IsManualManageDPIChanges())
	{
		FSlateApplication::Get().HandleDPIScaleChanged(GetNativeWindow().ToSharedRef());
	}
	
	Device->ScaleResolution(ScaleFactor, DPIScaleFactor, bClampWindowSizeState);

	UpdateWindow();
}

void SPIEPreviewWindow::OnWindowMoved(const TSharedRef<SWindow>& Window)
{
	float CurrentScaleFactor = GetWindowScaleFactor();
	ScaleWindow(CurrentScaleFactor);

	// save the position so we can restore it if the session is restarted
	FVector2D WindowPos = GetPositionInScreen();

	auto* Settings = GetMutableDefault<UPIEPreviewSettings>();
	Settings->WindowPosX = FMath::CeilToInt32(WindowPos.X);
	Settings->WindowPosY = FMath::CeilToInt32(WindowPos.Y);
	Settings->SaveConfig();
}

void SPIEPreviewWindow::OnDisplayDPIChanged(TSharedRef<SWindow> Window)
{
	float CurrentScaleFactor = GetWindowScaleFactor();
	SetWindowScaleFactor(CurrentScaleFactor);
}

float SPIEPreviewWindow::ComputeDPIScaleFactor()
{
	FVector2D WindowPos = GetPositionInScreen();
	int32 PointX = FMath::RoundToInt32(WindowPos.X);
	int32 PointY = FMath::RoundToInt32(WindowPos.Y);

	float DPIFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint((float)PointX, (float)PointY);

	return DPIFactor;
}

float SPIEPreviewWindow::ComputeScaleToDeviceSizeFactor() const
{
	float OutScreenFactor = 1.0f;

	FDisplayMetrics DisplayMetrics;
	FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);

	FVector2D WindowPos = GetPositionInScreen();
	int32 PointX = FMath::RoundToInt32(WindowPos.X);
	int32 PointY = FMath::RoundToInt32(WindowPos.Y);

	FPlatformRect& VirtualDisplayRect = DisplayMetrics.VirtualDisplayRect;
	PointX = FMath::Clamp(PointX, VirtualDisplayRect.Left, VirtualDisplayRect.Right);
	PointY = FMath::Clamp(PointY, VirtualDisplayRect.Top, VirtualDisplayRect.Bottom);

	float RatioMonitorResolution = 1.0f;
	int32 LocalPPI = 0;

	TArray<FMonitorInfo>& MonitorInfoArray = DisplayMetrics.MonitorInfo;
	for (int32 i = 0; i < MonitorInfoArray.Num(); ++i)
	{
		FMonitorInfo& MonitorInfo = MonitorInfoArray[i];

		const int32 PointOffset = 0;
		if (PointX >= MonitorInfo.DisplayRect.Left &&
			PointX <= MonitorInfo.DisplayRect.Right &&
			PointY >= MonitorInfo.DisplayRect.Top &&
			PointY <= MonitorInfo.DisplayRect.Bottom)
		{
			int32 MonitorWidth = MonitorInfo.DisplayRect.Right - MonitorInfo.DisplayRect.Left;
			int32 MonitorHeight = MonitorInfo.DisplayRect.Bottom - MonitorInfo.DisplayRect.Top;

			float MonitorResolutionScale = FMath::Min((float)MonitorWidth / (float)MonitorInfo.NativeWidth, (float)MonitorHeight / (float)MonitorInfo.NativeHeight);

			float NativeRatio = (float)MonitorInfo.NativeWidth / (float)MonitorInfo.NativeHeight;
			float CurrentRatio = (float)MonitorWidth / (float)MonitorHeight;
			float MonitorPixelRatio = NativeRatio / CurrentRatio;

			RatioMonitorResolution = MonitorResolutionScale * MonitorPixelRatio;

			LocalPPI = MonitorInfo.DPI;

			break;
		}
	}

	const int32 DevicePPI = Device->GetDeviceSpecs()->PPI;
	float PPIRatio = (!!DevicePPI && !!LocalPPI) ? (float)LocalPPI / (float)DevicePPI : 1.0f;
	OutScreenFactor = PPIRatio * RatioMonitorResolution;

	return OutScreenFactor;
}

void SPIEPreviewWindow::RotateWindow()
{
	if (!Device.IsValid())
	{
		return;
	}

	Device->SwitchOrientation(bClampWindowSizeState);

	UpdateGameLayerManagerDefaultViewport();

	UpdateWindow();
}

void SPIEPreviewWindow::FlipBezelVisibility()
{
	if (!Device.IsValid())
	{
		return;
	}

	bool bVisible = Device->GetBezelVisibility();
	Device->SetBezelVisibility(!bVisible, bClampWindowSizeState);

	UpdateWindow();
}

bool SPIEPreviewWindow::GetBezelVisibility() const
{
	bool bVisible = false;

	if (Device.IsValid())
	{
		bVisible = Device->GetBezelVisibility();
	}

	return bVisible;
}

void SPIEPreviewWindow::UpdateWindow()
{
	if (!Device.IsValid())
	{
		return;
	}

	// compute position window position
	// try to maintain its old top left corner position, but while keeping it inside the desktop area
	FVector2D WindowPos = GetPositionInScreen();
	int32 PosX = (int32)WindowPos.X;
	int32 PosY = (int32)WindowPos.Y;

	FDisplayMetrics DisplayMetrics;
	FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);

	if (PosX + Device->GetWindowWidth() > DisplayMetrics.VirtualDisplayRect.Right)
	{
		PosX = DisplayMetrics.VirtualDisplayRect.Right - Device->GetWindowWidth();
	}
	PosX = FMath::Max(DisplayMetrics.VirtualDisplayRect.Left, PosX);

	if (PosY + Device->GetWindowHeight() > DisplayMetrics.VirtualDisplayRect.Bottom)
	{
		PosY = DisplayMetrics.VirtualDisplayRect.Bottom - Device->GetWindowHeight() - FMath::TruncToInt32(SWindowDefs::DefaultTitleBarSize);
	}
	PosY = FMath::Max(DisplayMetrics.VirtualDisplayRect.Top, PosY);

	ReshapeWindow(FVector2D(PosX, PosY), FVector2D(Device->GetWindowWidth(), Device->GetWindowHeight()));

	// offset the viewport widget into its correct location
	ContentSlot->SetPadding(Device->GetViewportMargin());

	// bezel orientation depends on the window size so we need to call it after ReshapeWindow()
	ComputeBezelOrientation();
}

void SPIEPreviewWindow::SetDevice(TSharedPtr<FPIEPreviewDevice> InDevice)
{
	Device = InDevice;

	if (Device.IsValid())
	{
		CreatePIEPreviewBezelOverlay(InDevice->GetBezelTexture());
	}
}

void SPIEPreviewWindow::PrepareShutdown()
{
	SetOnWindowMoved(nullptr);

	if (HandleDPIChange.IsValid())
	{
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().OnSystemSignalsDPIChanged().Remove(HandleDPIChange);
		}
	}

	if (BezelImage.IsValid())
	{
		RemoveOverlaySlot(BezelImage.ToSharedRef());

		BezelBrush.SetResourceObject(nullptr);
		BezelImage = nullptr;
	}

	Device = nullptr;
}

int32 SPIEPreviewWindow::GetDefaultTitleBarSize()
{
	return FMath::TruncToInt32(SWindowDefs::DefaultTitleBarSize);
}

void SPIEPreviewWindow::SetGameLayerManagerWidget(TSharedPtr<class SGameLayerManager> GameLayerManager)
{
	GameLayerManagerWidget = GameLayerManager;

	UpdateGameLayerManagerDefaultViewport();
}

void SPIEPreviewWindow::UpdateGameLayerManagerDefaultViewport()
{
	if (Device.IsValid() && GameLayerManagerWidget.IsValid())
	{
		FIntPoint DeviceResolution;
		Device->GetDeviceDefaultResolution(DeviceResolution.X, DeviceResolution.Y);

		if (Device->IsDeviceFlipped())
		{
			Swap(DeviceResolution.X, DeviceResolution.Y);
		}

		GameLayerManagerWidget->SetUseFixedDPIValue(true, DeviceResolution);
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif