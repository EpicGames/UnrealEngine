// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Widgets/SWindow.h"
#include "PIEPreviewDevice.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class SPIEPreviewWindowTitleBar;

class UE_DEPRECATED(5.8, "PIEPreviewDeviceProfileSelector is deprecated and will be removed. Please use the new Preview Json System to preview Devices") SPIEPreviewWindow
	: public SWindow
{
public:
	/**
	* Default constructor. Use SNew(SPIEPreviewWindow) instead.
	*/
	PIEPREVIEWDEVICEPROFILESELECTOR_API SPIEPreviewWindow();

	PIEPREVIEWDEVICEPROFILESELECTOR_API ~SPIEPreviewWindow();

	// call this function set the 'preview device' and set initial values for the position and resolution scaling factor
	PIEPREVIEWDEVICEPROFILESELECTOR_API void PrepareWindow(FVector2D WindowPosition, const float InitialScaleFactor, TSharedPtr<FPIEPreviewDevice> PreviewDevice);

	// call this function to perform cleanup
	PIEPREVIEWDEVICEPROFILESELECTOR_API void PrepareShutdown();

	// retrieves the display device
	PIEPREVIEWDEVICEPROFILESELECTOR_API TSharedPtr<class FPIEPreviewDevice> GetDevice();

	// call to retrieve the window current resolution scale factor
	PIEPREVIEWDEVICEPROFILESELECTOR_API float GetWindowScaleFactor() const;

	// call to set the window resolution scale factor
	PIEPREVIEWDEVICEPROFILESELECTOR_API void SetWindowScaleFactor(const float ScaleFactor, const bool bStore = true);

	// call to rotate the window emulating a rotation of the physical device
	PIEPREVIEWDEVICEPROFILESELECTOR_API void RotateWindow();

	// wrapper function that will query the same functionality on the device
	PIEPREVIEWDEVICEPROFILESELECTOR_API bool IsRotationAllowed() const;

	// call to enable/disable bezel visibility
	PIEPREVIEWDEVICEPROFILESELECTOR_API void FlipBezelVisibility();

	// call to determine whether or not the phone bezel is displayed 
	PIEPREVIEWDEVICEPROFILESELECTOR_API bool GetBezelVisibility() const;

	static PIEPREVIEWDEVICEPROFILESELECTOR_API int32 GetDefaultTitleBarSize();

	// call with a true argument to restrict the window size to the desktop size
	PIEPREVIEWDEVICEPROFILESELECTOR_API void SetClampWindowSize(const bool bClamp);
	PIEPREVIEWDEVICEPROFILESELECTOR_API bool IsClampingWindowSize() const;

	/** creates and returns the settings menu */
	PIEPREVIEWDEVICEPROFILESELECTOR_API TSharedRef<SWidget> BuildSettingsMenu();

	/** we need the game layer manager to control the DPI scaling behavior and this function can be called should be called when the manager is available */
	PIEPREVIEWDEVICEPROFILESELECTOR_API void SetGameLayerManagerWidget(TSharedPtr<class SGameLayerManager> GameLayerManager);

private:
	PIEPREVIEWDEVICEPROFILESELECTOR_API virtual EHorizontalAlignment GetTitleAlignment() override;
	PIEPREVIEWDEVICEPROFILESELECTOR_API virtual TSharedRef<SWidget> MakeWindowTitleBar(const TSharedRef<SWindow>& Window, const TSharedPtr<SWidget>& CenterContent, EHorizontalAlignment CenterContentAlignment) override;

	// sets the device displayed by this window
	PIEPREVIEWDEVICEPROFILESELECTOR_API void SetDevice(TSharedPtr<class FPIEPreviewDevice> InDevice);

	// returns whether or not the provided ScaleFactor is a 'scale to physical device size' factor
	PIEPREVIEWDEVICEPROFILESELECTOR_API bool IsScalingToDeviceSizeFactor(const float ScaleFactor) const;

	// returns the scaling value reserved for 'scale to physical device size' factor
	PIEPREVIEWDEVICEPROFILESELECTOR_API float GetScaleToDeviceSizeFactor() const;

	// utility function used to correct a given window position if it's out of the display area
	PIEPREVIEWDEVICEPROFILESELECTOR_API void ValidatePosition(FVector2D& WindowPos);

	// call to perform resolution scaling on the device
	PIEPREVIEWDEVICEPROFILESELECTOR_API void ScaleWindow(float ScaleFactor);

	// this will properly scale and rotate the bezel to match the orientation of the device
	PIEPREVIEWDEVICEPROFILESELECTOR_API void ComputeBezelOrientation();

	// function that adds a widget to display the device's bezel
	PIEPREVIEWDEVICEPROFILESELECTOR_API void CreatePIEPreviewBezelOverlay(UTexture2D* pBezelImage);

	/** helper function used to create a menu item formed from a description text and a checkbox */
	/** @ IsCheckedFunction - computes the state of the checkbox */
	/** @ ExecuteActionFunction - function to be called when this menu item is clicked */
	PIEPREVIEWDEVICEPROFILESELECTOR_API void CreateMenuEntry(class FMenuBuilder& MenuBuilder, FText&& TextEntry, TFunction<ECheckBoxState()>&& IsCheckedFunction, TFunction<void()>&& ExecuteActionFunction);

	/** helper function used to create a simple text widget to the provided menu */
	/** @ CreateTextFunction - function should return the menu text that we want to display */
	PIEPREVIEWDEVICEPROFILESELECTOR_API void CreateTextMenuEntry(class FMenuBuilder& MenuBuilder, TFunction<FText()>&& CreateTextFunction);

	// call to compute window size, position on screen and bezel orientation
	PIEPREVIEWDEVICEPROFILESELECTOR_API void UpdateWindow();

	// utility function that will compute a screen and DPI scale factor needed to scale the display window to the physical device size
	PIEPREVIEWDEVICEPROFILESELECTOR_API float ComputeScaleToDeviceSizeFactor() const;

	// utility function used to retrieve the DPI scaling factor based on window's position
	PIEPREVIEWDEVICEPROFILESELECTOR_API float ComputeDPIScaleFactor();

	// callback used when DPI value varies, useful when we need to constrain the window to the physical device size
	PIEPREVIEWDEVICEPROFILESELECTOR_API void OnWindowMoved(const TSharedRef<SWindow>& Window);

	// callback used when DPI value varies, useful when we need to constrain the window to the physical device size
	PIEPREVIEWDEVICEPROFILESELECTOR_API void OnDisplayDPIChanged(TSharedRef<SWindow> Window);

	// utility function used to control game layer manager's DPI scaling behavior
	// this should be called every time the window is rotated
	PIEPREVIEWDEVICEPROFILESELECTOR_API void UpdateGameLayerManagerDefaultViewport();

protected:
	// brush created to display the bezel
	FSlateBrush BezelBrush;

	// pointer to the actual bezel image
	TSharedPtr<SImage> BezelImage;

	// pointer to the device that this window will display
	TSharedPtr<class FPIEPreviewDevice> Device;

	float CachedScaleToDeviceFactor = 0.0f;
	float CachedDPIScaleFactor = 0.0f;

	/** when true the window size will be restricted to the desktop size  */
	bool bClampWindowSizeState = true;

	float WindowScalingFactor = 0.0f;

	TSharedPtr<SPIEPreviewWindowTitleBar> WindowTitleBar;

	FDelegateHandle HandleDPIChange;

	/** Pointer to the game layer manager widget. This is needed because we want to control the DPI scaling behavior */
	TSharedPtr<SGameLayerManager> GameLayerManagerWidget;
};

FORCEINLINE bool SPIEPreviewWindow::IsRotationAllowed() const
{
	if (!Device.IsValid())
	{
		return false;
	}

	return Device->IsRotationAllowed();
}

FORCEINLINE TSharedPtr<class FPIEPreviewDevice> SPIEPreviewWindow::GetDevice()
{
	return Device;
}

FORCEINLINE void SPIEPreviewWindow::SetClampWindowSize(const bool bClamp)
{
	bClampWindowSizeState = bClamp;
}

FORCEINLINE bool SPIEPreviewWindow::IsClampingWindowSize() const
{
	return bClampWindowSizeState;
}

FORCEINLINE bool SPIEPreviewWindow::IsScalingToDeviceSizeFactor(const float ScaleFactor) const
{
	float DeviceSizeFactor = GetScaleToDeviceSizeFactor();
	return FMath::IsNearlyEqual(ScaleFactor, DeviceSizeFactor);
}

FORCEINLINE float SPIEPreviewWindow::GetWindowScaleFactor() const
{
	return WindowScalingFactor;
}

FORCEINLINE float SPIEPreviewWindow::GetScaleToDeviceSizeFactor() const
{
	return 0.0f;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
