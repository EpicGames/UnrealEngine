// Copyright Epic Games, Inc. All Rights Reserved.

#include "PIEPreviewWindowCoreStyle.h"
#include "Styling/CoreStyle.h"
#include "PIEPreviewWindowStyle.h"
#include "SlateGlobals.h"
#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "PIEPreviewDeviceSpecification.h"

#if WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS

TSharedPtr< ISlateStyle > FPIEPreviewWindowCoreStyle::Instance = nullptr;

#define RootToContentDir Style->RootToContentDir

TSharedRef<class ISlateStyle> FPIEPreviewWindowCoreStyle::Create(const FName& InStyleSetName)
{
	TSharedRef<FSlateStyleSet> Style = MakeShareable(new FSlateStyleSet(InStyleSetName));
	Style->SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FButtonStyle Button = FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("Button");

	const FButtonStyle ScreenRotationButtonStyle = FButtonStyle(Button)
		.SetDisabled(IMAGE_BRUSH("Icons/PIEWindow/WindowButton_Screen_Rotation_Disabled", FVector2D(23.0f, 18.0f)))
		.SetNormal(IMAGE_BRUSH("Icons/PIEWindow/WindowButton_Screen_Rotation_Normal", FVector2D(23.0f, 18.0f)))
		.SetHovered(IMAGE_BRUSH("Icons/PIEWindow/WindowButton_Screen_Rotation_Hovered", FVector2D(23.0f, 18.0f)))
		.SetPressed(IMAGE_BRUSH("Icons/PIEWindow/WindowButton_Screen_Rotation_Pressed", FVector2D(23.0f, 18.0f)));

	Style->Set("PIEWindow", FPIEPreviewWindowStyle()
		.SetScreenRotationButtonStyle(ScreenRotationButtonStyle));

	Style->Set("PIEWindow.Font", DEFAULT_FONT("Bold", 9));

	Style->Set("PIEWindow.MenuButton", FButtonStyle(Button)
		.SetNormal(BOX_BRUSH("Icons/PIEWindow/SmallRoundedButton", FMargin(7.f / 16.f), FLinearColor(1, 1, 1, 0.75f)))
		.SetHovered(BOX_BRUSH("Icons/PIEWindow/SmallRoundedButton", FMargin(7.f / 16.f), FLinearColor(1, 1, 1, 1.0f)))
		.SetPressed(BOX_BRUSH("Icons/PIEWindow/SmallRoundedButton", FMargin(7.f / 16.f))));

	Style->Set("ComboButton.Arrow", new IMAGE_BRUSH("Common/ComboArrow", FVector2D(8.0f, 8.0f)));

	return Style;
}

void FPIEPreviewWindowCoreStyle::InitializePIECoreStyle()
{
	if (!Instance.IsValid())
	{
		Instance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*Instance.Get());
	}
}

#undef RootToContentDir

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif