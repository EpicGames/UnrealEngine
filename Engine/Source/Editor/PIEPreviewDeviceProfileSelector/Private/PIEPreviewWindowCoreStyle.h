// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Styling/SlateStyle.h"

class UE_DEPRECATED(5.8, "PIEPreviewDeviceProfileSelector is deprecated and will be removed. Please use the new Preview Json System to preview Devices") FPIEPreviewWindowCoreStyle
{

public:

	static PIEPREVIEWDEVICEPROFILESELECTOR_API TSharedRef<class ISlateStyle> Create(const FName& InStyleSetName = "FPIECoreStyle");
	
	static PIEPREVIEWDEVICEPROFILESELECTOR_API void InitializePIECoreStyle();
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** @return the singleton instance. */
	static const ISlateStyle& Get()
	{
		return *(Instance.Get());
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
private:
		
	static PIEPREVIEWDEVICEPROFILESELECTOR_API TSharedPtr< class ISlateStyle > Instance;
};
#endif
