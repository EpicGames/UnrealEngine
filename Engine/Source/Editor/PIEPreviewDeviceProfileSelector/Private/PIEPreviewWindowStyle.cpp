// Copyright Epic Games, Inc. All Rights Reserved.

#include "PIEPreviewWindowStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PIEPreviewWindowStyle)

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FPIEPreviewWindowStyle::FPIEPreviewWindowStyle()
{
}

void FPIEPreviewWindowStyle::GetResources(TArray< const FSlateBrush* >& OutBrushes) const
{
	ScreenRotationButtonStyle.GetResources(OutBrushes);
	QuarterMobileContentScaleFactorButtonStyle.GetResources(OutBrushes);
	HalfMobileContentScaleFactorButtonStyle.GetResources(OutBrushes);
	FullMobileContentScaleFactorButtonStyle.GetResources(OutBrushes);
}

const FName FPIEPreviewWindowStyle::TypeName(TEXT("FPIEPreviewWindowStyle"));

const FPIEPreviewWindowStyle& FPIEPreviewWindowStyle::GetDefault()
{
	static FPIEPreviewWindowStyle Default;
	return Default;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
