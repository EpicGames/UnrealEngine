// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Styling/SlateTypes.h"
#include "PIEPreviewWindowStyle.generated.h"
/**
* Represents the appearance of an SPIEWindow
*/
USTRUCT()
struct UE_DEPRECATED(5.8, "PIEPreviewDeviceProfileSelector is deprecated and will be removed. Please use the new Preview Json System to preview Devices") FPIEPreviewWindowStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	PIEPREVIEWDEVICEPROFILESELECTOR_API FPIEPreviewWindowStyle();

	virtual ~FPIEPreviewWindowStyle() = default;

	PIEPREVIEWDEVICEPROFILESELECTOR_API virtual void GetResources(TArray< const FSlateBrush* >& OutBrushes) const override;

	static PIEPREVIEWDEVICEPROFILESELECTOR_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static PIEPREVIEWDEVICEPROFILESELECTOR_API const FPIEPreviewWindowStyle& GetDefault();
	
	/** Style used to draw the window ScreenRotationButton button */
	UPROPERTY()
	FButtonStyle ScreenRotationButtonStyle;
	FPIEPreviewWindowStyle& SetScreenRotationButtonStyle(const FButtonStyle& InScreenRotationButtonStyle) { ScreenRotationButtonStyle = InScreenRotationButtonStyle; return *this; }

	/** Style used to draw the window 0.25x button */
	UPROPERTY()
	FButtonStyle QuarterMobileContentScaleFactorButtonStyle;
	FPIEPreviewWindowStyle& SetQuarterMobileContentScaleFactorButtonStyle(const FButtonStyle& InQuarterMobileContentScaleFactorButtonStyle) { QuarterMobileContentScaleFactorButtonStyle = InQuarterMobileContentScaleFactorButtonStyle; return *this; }

	/** Style used to draw the window 0.5x button */
	UPROPERTY()
	FButtonStyle HalfMobileContentScaleFactorButtonStyle;
	FPIEPreviewWindowStyle& SetHalfMobileContentScaleFactorButtonStyle(const FButtonStyle& InHalfMobileContentScaleFactorButtonStyle) { HalfMobileContentScaleFactorButtonStyle = InHalfMobileContentScaleFactorButtonStyle; return *this; }

	/** Style used to draw the window 1X button */
	UPROPERTY()
	FButtonStyle FullMobileContentScaleFactorButtonStyle;
	FPIEPreviewWindowStyle& SetFullMobileContentScaleFactorButtonStyle(const FButtonStyle& InFullMobileContentScaleFactorButtonStyle) { FullMobileContentScaleFactorButtonStyle = InFullMobileContentScaleFactorButtonStyle; return *this; }

};
#endif
