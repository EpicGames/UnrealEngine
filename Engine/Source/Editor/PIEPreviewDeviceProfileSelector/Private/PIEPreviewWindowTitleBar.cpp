// Copyright Epic Games, Inc. All Rights Reserved.

#include "PIEPreviewWindowTitleBar.h"
#include "PIEPreviewWindow.h"
#include "Widgets/Input/SCheckBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SViewportToolBar.h"

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
/** Toolbar class used to add some menus to configure various device display settings */
class SPIEToolbar : public SViewportToolBar
{
public:
	SLATE_BEGIN_ARGS(SPIEToolbar) {}
		SLATE_EVENT(FOnGetContent, OnGetMenuContent)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	FReply OnMenuClicked();

protected:
	TSharedPtr<SMenuAnchor> MenuAnchor;
};

void SPIEToolbar::Construct(const FArguments& InArgs)
{
	SViewportToolBar::Construct(SViewportToolBar::FArguments());

	const FSlateBrush* ImageBrush = FPIEPreviewWindowCoreStyle::Get().GetBrush("ComboButton.Arrow");

	TSharedPtr<SWidget> ButtonContent =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 2.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.IsEnabled(true)
				.Cursor(EMouseCursor::Default)
				[
					SNew(SImage)
					.Image(ImageBrush)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];

	ChildSlot
	[
		SAssignNew(MenuAnchor, SMenuAnchor)
		.Padding(0)
		.Placement(MenuPlacement_BelowAnchor)
		[
			SNew(SButton)
			// Allows users to drag with the mouse to select options after opening the menu */
			.ClickMethod(EButtonClickMethod::MouseDown)
			.ContentPadding(FMargin(2.0f, 2.0f))
			.VAlign(VAlign_Center)
			.ButtonStyle(FPIEPreviewWindowCoreStyle::Get(), "PIEWindow.MenuButton")
			.OnClicked(this, &SPIEToolbar::OnMenuClicked)
			[
				ButtonContent.ToSharedRef()
			]
		]
		.OnGetMenuContent(InArgs._OnGetMenuContent)
	];
}

FReply SPIEToolbar::OnMenuClicked()
{
	// If the menu button is clicked toggle the state of the menu anchor which will open or close the menu
	if (MenuAnchor->ShouldOpenDueToClick())
	{
		MenuAnchor->SetIsOpen(true);
		SetOpenMenu(MenuAnchor);
	}
	else
	{
		MenuAnchor->SetIsOpen(false);
		TSharedPtr<SMenuAnchor> NullAnchor;
		SetOpenMenu(NullAnchor);
	}

	return FReply::Handled();
}


void SPIEPreviewWindowTitleBar::MakeTitleBarContentWidgets(TSharedPtr< SWidget >& OutLeftContent, TSharedPtr< SWidget >& OutRightContent)
{
	TSharedPtr< SWidget > OutRightContentBaseWindow;
	SWindowTitleBar::MakeTitleBarContentWidgets(OutLeftContent, OutRightContentBaseWindow);

	ScreenRotationButton = SNew(SButton)
		.IsFocusable(false)
		.IsEnabled(true)
		.ContentPadding(0.f)
		.OnClicked_Lambda
		(
			[this]()
			{
				TSharedPtr<SPIEPreviewWindow> PIEWindow = GetOwnerWindow();
				check(PIEWindow.IsValid());
				PIEWindow->RotateWindow();

				return FReply::Handled();
			}
		)
		.Cursor(EMouseCursor::Default)
		.ButtonStyle(FCoreStyle::Get(), "NoBorder")
		[
			SNew(SImage)
			.Image(this, &SPIEPreviewWindowTitleBar::GetScreenRotationButtonImage)
			.ColorAndOpacity(this, &SPIEPreviewWindowTitleBar::GetWindowTitleContentColor)
		]
		.IsEnabled_Lambda
		(
			[this]()
			{
				TSharedPtr<SPIEPreviewWindow> PIEWindow = GetOwnerWindow();
				check(PIEWindow.IsValid());
			
				return PIEWindow->IsRotationAllowed();
			}
		)
	;

	TSharedRef< SHorizontalBox > WindowTitleBarButtons =
	SNew(SHorizontalBox)
	.Visibility(EVisibility::SelfHitTestInvisible);

	WindowTitleBarButtons->AddSlot()
		.AutoWidth()
		[
			ScreenRotationButton.ToSharedRef()
		];

	// Add the settings menu widget
	WindowTitleBarButtons->AddSlot()
		.AutoWidth()
		[
			SNew(SPIEToolbar)
			.OnGetMenuContent_Lambda
			(
				[this]()
				{
					TSharedPtr<SPIEPreviewWindow> PIEWindow = GetOwnerWindow();
					check(PIEWindow.IsValid());

					return PIEWindow->BuildSettingsMenu();
				}
			)
		];

	if (OutRightContentBaseWindow.IsValid())
	{
		WindowTitleBarButtons->AddSlot()
			.AutoWidth()
			[
				OutRightContentBaseWindow.ToSharedRef()
			];
	}

	OutRightContent = SNew(SBox)
		.Visibility(EVisibility::SelfHitTestInvisible)
		.Padding(FMargin(2.0f, 0.0f, 0.0f, 0.0f))
		[
			WindowTitleBarButtons
		];
}

const FSlateBrush* SPIEPreviewWindowTitleBar::GetScreenRotationButtonImage() const
{
	TSharedPtr<SWindow> OwnerWindow = OwnerWindowPtr.Pin();
	
	if (!OwnerWindow.IsValid())
	{
		return nullptr;
	}

	if (ScreenRotationButton->IsPressed())
	{
		return &FPIEPreviewWindowCoreStyle::Get().GetWidgetStyle<FPIEPreviewWindowStyle>("PIEWindow").ScreenRotationButtonStyle.Pressed;
	}
	else if (ScreenRotationButton->IsHovered())
	{
		return &FPIEPreviewWindowCoreStyle::Get().GetWidgetStyle<FPIEPreviewWindowStyle>("PIEWindow").ScreenRotationButtonStyle.Hovered;
	}
	else
	{
		return &FPIEPreviewWindowCoreStyle::Get().GetWidgetStyle<FPIEPreviewWindowStyle>("PIEWindow").ScreenRotationButtonStyle.Normal;
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
