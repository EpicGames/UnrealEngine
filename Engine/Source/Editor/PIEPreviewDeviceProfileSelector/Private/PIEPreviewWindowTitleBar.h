// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Framework/Application/SWindowTitleBar.h"
#include "PIEPreviewWindow.h"
#include "PIEPreviewWindowStyle.h"
#include "PIEPreviewWindowCoreStyle.h"

/**
* Implements a window PIE title bar widget.
*/
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class UE_DEPRECATED(5.8, "PIEPreviewDeviceProfileSelector is deprecated and will be removed. Please use the new Preview Json System to preview Devices") SPIEPreviewWindowTitleBar
	: public SWindowTitleBar
{
private:
	/**
	* Creates widgets for this window's title bar area.
	*/
	PIEPREVIEWDEVICEPROFILESELECTOR_API virtual void MakeTitleBarContentWidgets(TSharedPtr< SWidget >& OutLeftContent, TSharedPtr< SWidget >& OutRightContent) override;

	PIEPREVIEWDEVICEPROFILESELECTOR_API const FSlateBrush* GetScreenRotationButtonImage() const;

	PIEPREVIEWDEVICEPROFILESELECTOR_API TSharedPtr<class SPIEPreviewWindow> GetOwnerWindow() const;

private:
	// Holds the screen rotation button.
	TSharedPtr<SButton> ScreenRotationButton;

	// Holds the clamp button.
	TSharedPtr<class SCheckBox> ClampWindowSizeCheckBox;
};

FORCEINLINE TSharedPtr<class SPIEPreviewWindow> SPIEPreviewWindowTitleBar::GetOwnerWindow() const
{
	TSharedPtr<SWindow> OwnerWindow = OwnerWindowPtr.Pin();
	TSharedPtr<SPIEPreviewWindow> PIEWindow = StaticCastSharedPtr<SPIEPreviewWindow>(OwnerWindow);

	return PIEWindow;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
