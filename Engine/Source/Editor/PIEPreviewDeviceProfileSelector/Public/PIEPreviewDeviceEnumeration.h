// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API PIEPREVIEWDEVICEPROFILESELECTOR_API

/**
* 
*/
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class UE_DEPRECATED(5.8, "PIEPreviewDeviceProfileSelector is deprecated and will be removed. Please use the new Preview Json System to preview Devices") FPIEPreviewDeviceContainerCategory
{
public:
	FPIEPreviewDeviceContainerCategory(const FString& InSubDirectoryPath, FText DisplayName) : 
		CategoryDisplayName(DisplayName),
		SubDirectoryPath(InSubDirectoryPath)
	{
	}

	UE_API FName GetCategoryName();
	UE_API FText GetCategoryToolTip();
	const FText& GetCategoryDisplayName() const { return CategoryDisplayName; }
	const FString& GetSubDirectoryPath() const { return SubDirectoryPath; }
	const TArray<TSharedPtr<FPIEPreviewDeviceContainerCategory>>& GetSubCategories() const {return SubCategories;}
	int GetDeviceStartIndex() const { return DeviceStartIndex; }
	int GetDeviceCount() const { return DeviceCount; }
protected:
	int32 DeviceStartIndex;
	int32 DeviceCount;
	FText CategoryDisplayName;
	FString SubDirectoryPath;
	TArray<TSharedPtr<FPIEPreviewDeviceContainerCategory>> SubCategories;
	friend class FPIEPreviewDeviceContainer;
};

class UE_DEPRECATED(5.8, "PIEPreviewDeviceProfileSelector is deprecated and will be removed. Please use the new Preview Json System to preview Devices") FPIEPreviewDeviceContainer
{
public:
	// Recursively iterate through 'RootDir' for all device json files.
	// Sub directories are recorded as categories.
	UE_API void EnumerateDeviceSpecifications(const FString& RootDir);

	const TSharedPtr<FPIEPreviewDeviceContainerCategory> GetRootCategory() const { return RootCategory; }

	const TArray<FString>& GetDeviceSpecifications() const { return DeviceSpecifications; }

	const TArray<FString>& GetDeviceSpecificationsLocalizedName() const { return DeviceSpecificationsLocalizedName; }

	// return the category that contains DeviceIndex.
	UE_API const TSharedPtr<FPIEPreviewDeviceContainerCategory> FindDeviceContainingCategory(int32 DeviceIndex) const;

private:
	FString DeviceSpecificationRootDir;
	FString GetDeviceSpecificationRootDir() const { return DeviceSpecificationRootDir; }
	TSharedPtr<FPIEPreviewDeviceContainerCategory> RootCategory;

	// All device specifications found.
	TArray<FString> DeviceSpecifications;
	TArray<FString> DeviceSpecificationsLocalizedName;
	UE_API void EnumerateDeviceSpecifications(TSharedPtr<FPIEPreviewDeviceContainerCategory> SubCategory);
	UE_API void EnumerateDeviceLocalizedNames();
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#undef UE_API
