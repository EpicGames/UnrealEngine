// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "IDeviceProfileSelectorModule.h"
#include "PIEPreviewDeviceEnumeration.h"
#include "RHIDefinitions.h"
#include "Misc/CommandLine.h"
#include "Widgets/SWindow.h"
#include "IPIEPreviewDeviceModule.h"
#include "Dom/JsonObject.h"
/**
* Implements the Preview Device Profile Selector module.
*/
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class UE_DEPRECATED(5.8, "PIEPreviewDeviceProfileSelector is deprecated and will be removed. Please use the new Preview Json System to preview Devices") FPIEPreviewDeviceModule
	: public IPIEPreviewDeviceModule
{
public:
	FPIEPreviewDeviceModule() : bInitialized(false)
	{
	}

	//~ Begin IDeviceProfileSelectorModule Interface
	virtual const FString GetRuntimeDeviceProfileName() override;
	virtual bool GetSelectorPropertyValue(const FName& PropertyType, FString& PropertyValueOUT) override;
	//~ End IDeviceProfileSelectorModule Interface

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface

	/**
	* Virtual destructor.
	*/
	virtual ~FPIEPreviewDeviceModule() = default;

	virtual FName GetPreviewPlatformName() override;

	virtual void ApplyPreviewDeviceState() override;
	
	virtual TSharedRef<SWindow> CreatePIEPreviewDeviceWindow(FVector2D ClientSize, FText WindowTitle, EAutoCenter AutoCenterType, FVector2D ScreenPosition, TOptional<float> MaxWindowWidth, TOptional<float> MaxWindowHeight) override;

	/** call this after the window is created and registered to the application to setup display related parameters */
	virtual void OnWindowReady(TSharedRef<SWindow> Window) override;
	
	virtual const FPIEPreviewDeviceContainer& GetPreviewDeviceContainer() ;
	TSharedPtr<FPIEPreviewDeviceContainerCategory> GetPreviewDeviceRootCategory() const { return EnumeratedDevices.GetRootCategory(); }

	static bool IsRequestingPreviewDevice()
	{
		FString PreviewDeviceDummy;
		return FParse::Value(FCommandLine::Get(), GetPreviewDeviceCommandSwitch(), PreviewDeviceDummy);
	}

	/** we need the game layer manager to control the DPI scaling behavior and this function can be called should be called when the manager is available */
	virtual void SetGameLayerManagerWidget(TSharedPtr<class SGameLayerManager> GameLayerManager) override;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FAddToDevicePreviewMenu, const FText& DeviceCategoryName, class FMenuBuilder& MenuBuilder);
	FAddToDevicePreviewMenu AddToDevicePreviewMenuDelegates;

	virtual void SetPreviewDevice(const FString& DeviceName) override;

private:
	static const TCHAR* GetPreviewDeviceCommandSwitch()
	{
		return TEXT("MobileTargetDevice=");
	}

	/** callback function registered in UGameViewportClient::OnViewportCreated needed to disable mouse capture/lock	*/
	void OnViewportCreated();

	/** callback function registered in FCoreDelegates::OnFEngineLoopInitComplete needed to position and show the window */
	void OnEngineInitComplete();

	void InitPreviewDevice();
	static FString GetDeviceSpecificationContentDir();
	bool ReadDeviceSpecification();
	FString FindDeviceSpecificationFilePath(const FString& SearchDevice);

	void UpdateDisplayResolution();

	/** this function will attempt to load the last known window position and scaling factor */
	bool ReadWindowConfig();

private:
	bool bInitialized;
	FString DeviceProfile;
	FString PreviewDevice;
	TSharedPtr<FJsonObject> JsonRootObject;

	/** delegate handle that will be obtained from UGameViewportClient::OnViewportCreated */ 
	FDelegateHandle ViewportCreatedDelegate;

	/** delegate handle that will be obtained from FCoreDelegates::OnFEngineLoopInitComplete */
	FDelegateHandle EngineInitCompleteDelegate;

	FPIEPreviewDeviceContainer EnumeratedDevices;

	TSharedPtr<class FPIEPreviewDevice> Device;

	TWeakPtr<class SPIEPreviewWindow> WindowWPtr;

	FVector2D InitialWindowPosition;
	float InitialWindowScaleValue;

	TSharedPtr<class SGameLayerManager> GameLayerManagerWidget;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS